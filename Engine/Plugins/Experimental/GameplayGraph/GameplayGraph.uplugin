{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Gameplay Graph",
	"Description" : "A graph representation model and common graph alogrithms that can be used for gameplay.",
	"Category": "Gameplay",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "http://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"ExplicitlyLoaded": true,
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Modules": [
		{
			"Name": "GameplayGraph",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
	]
}
