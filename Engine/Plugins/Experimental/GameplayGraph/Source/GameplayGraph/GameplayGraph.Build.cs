// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class GameplayGraph : ModuleRules
	{
        public GameplayGraph(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject"
				}
			);
		}
	}
}
