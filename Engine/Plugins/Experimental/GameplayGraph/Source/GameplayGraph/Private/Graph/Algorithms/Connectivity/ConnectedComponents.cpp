// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/Algorithms/Connectivity/ConnectedComponents.h"
#include "Graph/Algorithms/Search/Search.h"

namespace Graph::Algorithms
{

	TArray<TSet<FGraphVertexHandle>> FindConnectedComponents(const TSet<FGraphVertexHandle>& StartingSet)
	{
		UE_LOGF(LogGameplayGraph, Verbose, "FindConnectedComponents Starting Set Size: %d", StartingSet.Num());
		TArray<TSet<FGraphVertexHandle>> Output;
		// Do a BFS from each node already assigned to an island.
		// Every node that we hit that's also in the StartingSet gets
		// added to the connected component we're in the middle of
		// building. When the traversal stops, and there's still
		// un-touched nodes in the starting set, we create a new
		// connected component and start again.
		TSet<FGraphVertexHandle> WorkQueue = StartingSet;
		while (!WorkQueue.IsEmpty())
		{
			TSet<FGraphVertexHandle> CurrentComponent;
			BFS(
				*WorkQueue.CreateConstIterator(),
				[&CurrentComponent, &WorkQueue, &StartingSet](const FGraphVertexHandle& Handle)
				{
					if (StartingSet.Contains(Handle))
					{
						CurrentComponent.Add(Handle);
					}
					WorkQueue.Remove(Handle);
					return false;
				}
			);

			if (CurrentComponent.IsEmpty())
			{
				// Fallback to prevent adding infinitely many empty components into the array.
				UE_LOGF(LogGameplayGraph, Warning, "FindConnectedComponents found an empty component with a non-empty work queue [probably due to an incomplete handle?]");
				break;
			}

			Output.Add(CurrentComponent);
		}

		return Output;
	}

}