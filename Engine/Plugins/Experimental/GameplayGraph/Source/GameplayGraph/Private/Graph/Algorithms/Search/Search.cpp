// Copyright Epic Games, Inc. All Rights Reserved.
#include "Graph/Algorithms/Search/Search.h"

#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/Set.h"
#include "Graph/GraphVertex.h"

namespace Graph::Algorithms
{
	namespace
	{
		using BFSDataStructure = TQueue<FGraphVertexHandle>;
		using DFSDataStructure = TArray<FGraphVertexHandle>;

		template<typename TDataStructure>
		FGraphVertexHandle GetNextAndAdvance(TDataStructure& Queue)
		{
			if constexpr (std::is_same_v<TDataStructure, BFSDataStructure>)
			{
				FGraphVertexHandle Ret;
				if (ensure(Queue.Dequeue(Ret)))
				{
					return Ret;
				}
				return {};
			}
			else if constexpr (std::is_same_v<TDataStructure, DFSDataStructure>)
			{
				if (ensure(!Queue.IsEmpty()))
				{
					FGraphVertexHandle Next = Queue.Top();
					Queue.Pop();
					return Next;
				}

				return {};
			}
			else
			{
				static_assert("Invalid data structure for graph search [GetNextAndAdvance].");
				return {};
			}
		}

		template<typename TDataStructure>
		void AddToWorkQueue(TDataStructure& Queue, const FGraphVertexHandle& Handle)
		{
			if constexpr (std::is_same_v<TDataStructure, BFSDataStructure>)
			{
				Queue.Enqueue(Handle);
			}
			else if constexpr (std::is_same_v<TDataStructure, DFSDataStructure>)
			{
				Queue.Add(Handle);
			}
			else
			{
				static_assert("Invalid data structure for graph search [AddToWorkQueue].");
			}
		}

		template<typename TDataStructure>
		FGraphVertexHandle GenericSearch(const FGraphVertexHandle& Start, FSearchCallback Callback)
		{
			TDataStructure WorkQueue;
			AddToWorkQueue<TDataStructure>(WorkQueue, Start);

			TSet<FGraphVertexHandle> Seen;
			Seen.Add(Start);

			while (!WorkQueue.IsEmpty())
			{
				FGraphVertexHandle Next = GetNextAndAdvance<TDataStructure>(WorkQueue);
				if (Callback(Next))
				{
					return Next;
				}

				if (!Next.IsComplete())
				{
					continue;
				}

				// Get neighbors and add to the queue.
				if (TObjectPtr<UGraphVertex> Node = Next.GetVertex())
				{
					Node->ForEachAdjacentVertex(
						[&WorkQueue, &Seen](const FGraphVertexHandle& Neighbor)
						{
							if (Neighbor.IsComplete() && !Seen.Contains(Neighbor))
							{
								AddToWorkQueue<TDataStructure>(WorkQueue, Neighbor);
								Seen.Add(Neighbor);
							}
						}
					);
				}
			}

			return {};
		}

	}

	FGraphVertexHandle BFS(const FGraphVertexHandle& Start, FSearchCallback Callback)
	{
		return GenericSearch<BFSDataStructure>(Start, Callback);
	}

	FGraphVertexHandle DFS(const FGraphVertexHandle& Start, FSearchCallback Callback)
	{
		return GenericSearch<DFSDataStructure>(Start, Callback);
	}
}