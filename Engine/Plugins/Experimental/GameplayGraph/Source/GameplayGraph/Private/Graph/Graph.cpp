// Copyright Epic Games, Inc. All Rights Reserved.
#include "Graph/Graph.h"

#include "Graph/Algorithms/Connectivity/ConnectedComponents.h"
#include "Graph/GraphVertex.h"
#include "Graph/GraphSerialization.h"
#include "GenericPlatform/GenericPlatformMath.h"
#include "HAL/IConsoleManager.h"
#include "Templates/UnrealTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(Graph)

namespace Graph
{
	static bool bFixupGraphOnLoad = true;
	static FAutoConsoleVariableRef FixupGraphOnLoadCVar(TEXT("GameplayGraph.FixupGraphOnLoad"), bFixupGraphOnLoad, TEXT("Merge and split islands when loaded from serialization."));

	// Return a rough estimate of the total number of edges in the graph. This can be used to reserve memory for each edge.
	// The return type is int32 to match IGraphSerialization::Initialize().
	static int32 EstimateNumEdges(const UGraph& Graph)
	{
		const TMap<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& Vertices = Graph.GetVertices();
		if (Vertices.IsEmpty())
		{
			return 0;
		}

		const TObjectPtr<UGraphVertex>& FirstVertex = Vertices.CreateConstIterator()->Value;
		if (UNLIKELY(!FirstVertex))
		{
			return 0;
		}

		// Clamp estimate to reduce the chance of a specially crafted graph or atypical sample vertex leading to excessive overallocation.
		uint64 Estimate = static_cast<uint64>(Vertices.Num()) * FMath::Clamp(FirstVertex->NumEdges(), 2, 10);
		return IntCastChecked<int32>(FMath::Min<uint64>(Estimate, std::numeric_limits<int32>::max()));
	}
}

void UGraph::Empty()
{
	Vertices.Empty();
	Islands.Empty();
}

void UGraph::InitializeFromProperties(const FGraphProperties& InProperties)
{
	Empty();
	Properties = InProperties;
}

FGraphVertexHandle UGraph::CreateVertex(FGraphUniqueIndex InUniqueIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::CreateVertex);
	TObjectPtr<UGraphVertex> Vertex = CreateTypedVertex();
	if (!Vertex)
	{
		return {};
	}

	if (!ensure(InUniqueIndex.IsValid()))
	{
		return {};
	}

	Vertex->SetUniqueIndex(InUniqueIndex);
	Vertex->SetParentGraph(this);
	if (Vertices.Contains(Vertex->Handle()))
	{
		return {};
	}

	Vertex->OnCreate();
	RegisterVertex(Vertex);
	OnVertexCreated.Broadcast(Vertex->Handle());
	return Vertex->Handle();
}

void UGraph::ChangeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle)
{
	UGraphVertex* Vertex = GetSafeVertexFromHandle(OldVertexHandle);
	if (!ensure(Vertex))
	{
		return;
	}

	// Update vertex
	Vertex->SetUniqueIndex(NewVertexHandle.GetUniqueIndex());

	Vertices.Remove(OldVertexHandle);
	Vertices.Add(NewVertexHandle, Vertex);

	// Update edges. We use that it is bidirectional to only modify the edges of the adjacent vertices. 
	Vertex->ForEachAdjacentVertex(
		[this, &OldVertexHandle, &NewVertexHandle](const FGraphVertexHandle& EdgeVertexHandle)
		{
			UGraphVertex* EdgeNode = GetSafeVertexFromHandle(EdgeVertexHandle);
			if (ensure(EdgeNode))
			{
				EdgeNode->ChangeEdgeVertexHandle(OldVertexHandle, NewVertexHandle);
			}
		});

	// Update island
	if (UGraphIsland* Island = Vertex->GetParentIsland().GetIsland())
	{
		Island->ChangeVertexHandle(OldVertexHandle, NewVertexHandle);
	}
}

void UGraph::RegisterVertex(TObjectPtr<UGraphVertex> Vertex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RegisterVertex);
	if (!Vertex)
	{
		return;
	}

	Vertex->SetParentGraph(this);
	const FGraphVertexHandle Handle = Vertex->Handle();
	Vertices.Add(Handle, Vertex);
}

bool UGraph::CreateEdge(FGraphVertexHandle Node1, FGraphVertexHandle Node2, bool bMergeIslands)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::CreateEdge);
	if (!Node1.IsComplete() || !Node2.IsComplete())
	{
		return false;
	}

	if (Node2 < Node1)
	{
		std::swap(Node1, Node2);
	}

	UGraphVertex* Node1Ptr = Node1.GetVertex();
	UGraphVertex* Node2Ptr = Node2.GetVertex();
	if (!Node1Ptr || !Node2Ptr)
	{
		return false;
	}

	// For edges, we also need to make sure the edge doesn't already exist.
	if (Node1Ptr->HasEdgeTo(Node2) || Node2Ptr->HasEdgeTo(Node1))
	{
		return false;
	}

	if (Properties.bGenerateIslands)
	{
		const FGraphIslandHandle& Island1 = Node1Ptr->GetParentIsland();
		const FGraphIslandHandle& Island2 = Node2Ptr->GetParentIsland();

		const UGraphIsland* Island1Ptr = Island1.GetIsland();
		const UGraphIsland* Island2Ptr = Island2.GetIsland();

		if (Island1Ptr && Island2Ptr)
		{
			// Possible merge scenario.
			const bool bIsMerge = Island1 != Island2;
			if (bIsMerge && (!Island1Ptr->IsOperationAllowed(EGraphIslandOperations::Merge) || !Island2Ptr->IsOperationAllowed(EGraphIslandOperations::Merge)))
			{
				return {};
			}
		}
		else if (const UGraphIsland* RelevantIsland = Island1Ptr ? Island1Ptr : Island2Ptr)
		{
			// Regular add scenario.
			if (!RelevantIsland->IsOperationAllowed(EGraphIslandOperations::Add))
			{
				return {};
			}
		}
	}

	// Also need to make sure the nodes are aware of the new edge.
	Node1Ptr->AddEdgeTo(Node2);
	Node2Ptr->AddEdgeTo(Node1);
	OnEdgeCreated.Broadcast(FEdgeSpecifier{Node1, Node2});

	// If we want to keep track of islands, this is where we need to create/merge islands.
	if (Properties.bGenerateIslands)
	{
		// After we've created the edge - we want to see one of the two islands is invalid in which
		// case we have to initialize that node's island. This is needed or else we might not detect a merge
		// properly when creating multiple edges at a time in UGraph::CreateBulkEdges.
		const FGraphIslandHandle& IslandHandle1 = Node1Ptr->GetParentIsland();
		const FGraphIslandHandle& IslandHandle2 = Node2Ptr->GetParentIsland();

		if (!IslandHandle1.IsValid() || !IslandHandle2.IsValid())
		{
			if (UGraphIsland* Island1 = IslandHandle1.GetIsland())
			{
				Island1->AddVertex(Node2);
			}
			else if (UGraphIsland* Island2 = IslandHandle2.GetIsland())
			{
				Island2->AddVertex(Node1);
			}
		}

		if (bMergeIslands)
		{
			const FEdgeSpecifier Edge{ Node1, Node2 };
			MergeOrCreateIslands({ Edge });
		}
	}

	return true;
}

void UGraph::CreateBulkEdges(TArray<FEdgeSpecifier>&& NodesToConnect)
{
	TArray<FEdgeSpecifier> FinalEdges;
	FinalEdges.Reserve(NodesToConnect.Num());

	// Create all edges normally but don't call MergeOrCreateIslands yet. We'll use the bulk function instead.
	for (const FEdgeSpecifier& Params : NodesToConnect)
	{
		if (CreateEdge(Params.GetVertexHandle1(), Params.GetVertexHandle2(), false))
		{
			FinalEdges.Add(Params);
		}
	}

	// MergeOrCreateIslands incrementally determines island connectivity one edge at a time. This is efficient if we're handling a single edge but less
	// efficient if we're trying to add a bunch of edges all at the same time since it'll cause a vertex to jump between islands. The Bulk function helps prevent that.
	MergeOrCreateIslands(FinalEdges);
}

FGraphIslandHandle UGraph::CreateIsland(TArray<FGraphVertexHandle> InputNodes, FGraphUniqueIndex InUniqueIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::CreateIsland);
	if (InputNodes.IsEmpty())
	{
		return {};
	}

	TObjectPtr<UGraphIsland> Island = CreateTypedIsland();
	if (!Island)
	{
		return {};
	}
	Island->SetParentGraph(this);
	Island->OnCreate();
	Island->SetUniqueIndex((!InUniqueIndex.IsValid()) ? FGraphUniqueIndex::CreateUniqueIndex() : InUniqueIndex);
	RegisterIsland(Island);

	OnIslandCreated.Broadcast(Island->Handle());
	for (const FGraphVertexHandle& Node : InputNodes)
	{
		Island->AddVertex(Node);
	}

	return Island->Handle();
}

void UGraph::RegisterIsland(TObjectPtr<UGraphIsland> Island)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RegisterIsland);
	if (!Island)
	{
		return;
	}
	const FGraphIslandHandle Handle = Island->Handle();
	Island->SetParentGraph(this);
	Islands.Add(Handle, Island);
}

void UGraph::RemoveIsland(const FGraphIslandHandle& IslandHandle)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RemoveIsland);
	if (!IslandHandle.IsValid())
	{
		return;
	}

	if (UGraphIsland* Island = GetSafeIslandFromHandle(IslandHandle))
	{
		if (!Island->IsOperationAllowed(EGraphIslandOperations::Destroy))
		{
			return;
		}
		Island->Destroy();
	}
	Islands.Remove(IslandHandle);
}

void UGraph::MergeOrCreateIslands(const TArray<FEdgeSpecifier>& InEdges)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::MergeOrCreateIslands);
	if (InEdges.IsEmpty())
	{
		return;
	}

	// Every vertex needs to track which new island it's going to be in. This can either be an existing island or a completely new island.
	// This function guarantees that any vertex gets *actually* added into an island exactly once. We're guaranteed that after this function,
	// every affected vertex is in an island. So the FGraphIslandHandle either points to an existing island (in which case the vertex should be
	// added to that island) or an invalid island (in which case the vertex is going to be added into a new island). Note that we can assume that
	// the island IDs will never be negative (and -1 is reserved for "invalid") so we can represent temporary islands as having IDs of -2, -3, etc.
	TMap<FGraphVertexHandle, FGraphIslandHandle> VertexIslandChanges;
	VertexIslandChanges.Reserve(InEdges.Num());

	TMap<FGraphIslandHandle, TSet<FGraphVertexHandle>> IslandVertexAdditions;
	TMap<FGraphIslandHandle, int32> IslandSizeOverride;

	FGraphUniqueIndex NextTemporaryIslandId;
	NextTemporaryIslandId.SetTemporary(true);

	auto IsIslandTemporary = [](const FGraphIslandHandle& Handle)
	{
		return Handle.GetUniqueIndex().IsTemporary();
	};

	auto GetIslandSize = [&IslandVertexAdditions, &IsIslandTemporary, &IslandSizeOverride](const FGraphIslandHandle& Handle)
	{
		if (IsIslandTemporary(Handle))
		{
			return IslandVertexAdditions.FindRef(Handle).Num();
		}
		else if (int32* Size = IslandSizeOverride.Find(Handle))
		{
			return *Size;
		}
		else if (TObjectPtr<UGraphIsland> Island = Handle.GetIsland())
		{
			return Island->Num();
		}
		return 0;
	};

	auto RecacheIslandSize = [&IsIslandTemporary, &IslandVertexAdditions, &IslandSizeOverride](const FGraphIslandHandle& Handle, int32 Delta)
	{
		if (IsIslandTemporary(Handle))
		{
			return;
		}

		if (!IslandSizeOverride.Contains(Handle))
		{
			int32 Size = 0;
			if (TObjectPtr<UGraphIsland> Island = Handle.GetIsland())
			{
				Size += Island->Num();
			}

			IslandSizeOverride.Add(Handle, Size);
		}

		IslandSizeOverride.Add(Handle, IslandSizeOverride.FindRef(Handle) + Delta);
	};

	auto ForEveryVertexInIsland = [&IslandVertexAdditions]<typename TLambda>(const FGraphIslandHandle& IslandHandle, TLambda&& Func)
	{
		// We need to always iterate over all the vertices in IslandVertexAdditions.
		for (const FGraphVertexHandle& VertexHandle : IslandVertexAdditions.FindRef(IslandHandle))
		{
			Func(VertexHandle);
		}

		// Is this is a real non-temporary island, we need to iterate over its vertices too. Note that we don't need to
		// keep track of island vertex removals since we never do partial removal from an island. If an island is being removed,
		// all its vertices are being moved into a different island.
		if (TObjectPtr<UGraphIsland> Island = IslandHandle.GetIsland())
		{
			for (const FGraphVertexHandle& VertexHandle : Island->GetVertices())
			{
				Func(VertexHandle);
			}
		}
	};

	for (const FEdgeSpecifier& EdgeHandle : InEdges)
	{
		const FGraphVertexHandle& AHandle = EdgeHandle.GetVertexHandle1();
		const FGraphVertexHandle& BHandle = EdgeHandle.GetVertexHandle2();

		UGraphVertex* NodeA = AHandle.GetVertex();
		UGraphVertex* NodeB = BHandle.GetVertex();
		if (!NodeA || !NodeB)
		{
			return;
		}

		FGraphIslandHandle IslandHandleA = VertexIslandChanges.FindRef(AHandle);
		if (!IslandHandleA.IsValid())
		{
			IslandHandleA = NodeA->GetParentIsland();
		}

		FGraphIslandHandle IslandHandleB = VertexIslandChanges.FindRef(BHandle);
		if (!IslandHandleB.IsValid())
		{
			IslandHandleB = NodeB->GetParentIsland();
		}

		if (IslandHandleA.IsValid() && IslandHandleB.IsValid() && IslandHandleA != IslandHandleB)
		{
			// We need to move all the vertices in one island to the other.
			FGraphIslandHandle ToKeepIsland;
			FGraphIslandHandle ToRemoveIsland;

			const bool bIsATemporary = IsIslandTemporary(IslandHandleA);
			const bool bIsBTemporary = IsIslandTemporary(IslandHandleB);
			if (bIsATemporary == bIsBTemporary)
			{
				// In the case both islands are not temporary (or both are temporary), choose the larger island to add to.
				const int32 SizeA = GetIslandSize(IslandHandleA);
				const int32 SizeB = GetIslandSize(IslandHandleB);
				ToKeepIsland = (SizeA > SizeB) ? IslandHandleA : IslandHandleB;
				ToRemoveIsland = (SizeA > SizeB) ? IslandHandleB : IslandHandleA;
			}
			else
			{
				// In the case that only one of the islands is temporary, choose the non-temporary island.
				ToKeepIsland = bIsATemporary ? IslandHandleB : IslandHandleA;
				ToRemoveIsland = bIsATemporary ? IslandHandleA : IslandHandleB;
			}

			int32 VerticesChanged = 0;
			ForEveryVertexInIsland(ToRemoveIsland,
				[&VertexIslandChanges, &IslandVertexAdditions, &ToKeepIsland, &VerticesChanged](const FGraphVertexHandle& VertexHandle)
				{
					VertexIslandChanges.Add(VertexHandle, ToKeepIsland);
					IslandVertexAdditions.FindOrAdd(ToKeepIsland).Add(VertexHandle);
					++VerticesChanged;
				}
			);

			IslandVertexAdditions.Remove(ToRemoveIsland);
			RecacheIslandSize(ToKeepIsland, VerticesChanged);
			RecacheIslandSize(ToRemoveIsland, -VerticesChanged);
		}
		else if (IslandHandleA.IsValid() && !IslandHandleB.IsValid())
		{
			VertexIslandChanges.Add(BHandle, IslandHandleA);
			IslandVertexAdditions.FindOrAdd(IslandHandleA).Add(BHandle);
			RecacheIslandSize(IslandHandleA, 1);
		}
		else if (IslandHandleB.IsValid() && !IslandHandleA.IsValid())
		{
			VertexIslandChanges.Add(AHandle, IslandHandleB);
			IslandVertexAdditions.FindOrAdd(IslandHandleB).Add(AHandle);
			RecacheIslandSize(IslandHandleB, 1);
		}
		else if (!IslandHandleA.IsValid() && !IslandHandleB.IsValid())
		{
			// Neither is in an island - need to create a new one.
			FGraphIslandHandle NewIsland{ NextTemporaryIslandId.NextUniqueIndex(), nullptr};
			VertexIslandChanges.Add(AHandle, NewIsland);
			VertexIslandChanges.Add(BHandle, NewIsland);
			IslandVertexAdditions.Emplace(NewIsland, TSet<FGraphVertexHandle>{AHandle, BHandle});
		}
	}

	// Now that we have all the changes we want to make, we can start making them! Iterate over
	// VertexIslandChanges and handle additions into an existing island first.
	for (const TPair<FGraphVertexHandle, FGraphIslandHandle>& Change : VertexIslandChanges)
	{
		if (IsIslandTemporary(Change.Value))
		{
			continue;
		}

		if (TObjectPtr<UGraphIsland> Island = Change.Value.GetIsland())
		{
			Island->AddVertex(Change.Key);
		}
	}

	// Next handle the creation of any temporary islands.
	for (const TPair<FGraphIslandHandle, TSet<FGraphVertexHandle>>& Change : IslandVertexAdditions)
	{
		if (!IsIslandTemporary(Change.Key))
		{
			continue;
		}

		CreateIsland(Change.Value.Array());
	}

	// Destroy all empty islands
	for (const TPair<FGraphIslandHandle, int32>& Change : IslandSizeOverride)
	{
		if (IsIslandTemporary(Change.Key))
		{
			continue;
		}

		if (Change.Value == 0)
		{
			RemoveIsland(Change.Key);
		}
	}
}

void UGraph::RemoveVertex(const FGraphVertexHandle& NodeHandle)
{
	RemoveBulkVertices({ NodeHandle });
}

void UGraph::RemoveBulkVertices(const TArray<FGraphVertexHandle>& InHandles)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RemoveBulkVertices);

	TSet<FGraphIslandHandle> AffectedIslands;

	TArray<FGraphVertexHandle> AffectedVertices;
	AffectedVertices.Reserve(InHandles.Num());

	for (const FGraphVertexHandle& NodeHandle : InHandles)
	{
		if (UGraphVertex* Node = GetSafeVertexFromHandle(NodeHandle))
		{
			if (TObjectPtr<UGraphIsland> Island = Node->GetParentIsland().GetIsland())
			{
				AffectedIslands.Add(Node->GetParentIsland());
				Island->RemoveVertex(Node->Handle());
			}

			// We must remove every edge this node is a part of. Need to make a copy of the edges because
			// otherwise we're modifying the container during iteration over it in ForEachAdjacentVertex.
			TArray<FGraphVertexHandle> EdgeCopy;
			EdgeCopy.Reserve(Node->NumEdges());
			Node->ForEachAdjacentVertex(
				[&EdgeCopy](const FGraphVertexHandle& OtherNodeHandle)
				{
					EdgeCopy.Add(OtherNodeHandle);
				}
			);

			for (const FGraphVertexHandle& AdjacentVertexHandle : EdgeCopy)
			{
				// Don't immediately handle islands. We'll do it later.
				RemoveEdgeInternal(Node->Handle(), AdjacentVertexHandle, false);
			}

			AffectedVertices.Add(Node->Handle());
		}
	}

	if (Properties.bGenerateIslands)
	{
		for (const FGraphIslandHandle& IslandHandle : AffectedIslands)
		{
			RemoveOrSplitIsland(IslandHandle.GetIsland());
		}
	}

	// A final pass after generation of islands to clean up book-keeping.
	// TODO: Not sure if this is necessary and can be done before we regenerate islands?
	for (const FGraphVertexHandle& NodeHandle : AffectedVertices)
	{
		if (UGraphVertex* Node = NodeHandle.GetVertex())
		{
			Node->HandleOnVertexRemoved();
		}
		Vertices.Remove(NodeHandle);
	}
}

void UGraph::RemoveEdge(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB)
{
	RemoveEdgeInternal(VertexHandleA, VertexHandleB, true);
}

void UGraph::RemoveEdgeInternal(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB, bool bHandleIslands)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RemoveEdgeInternal);
	FGraphIslandHandle IslandHandle;

	// Need to remove the edge reference from both nodes.
	if (UGraphVertex* NodeA = VertexHandleA.GetVertex())
	{
		NodeA->RemoveEdge(VertexHandleB);
		IslandHandle = NodeA->GetParentIsland();
	}

	if (UGraphVertex* NodeB = VertexHandleB.GetVertex())
	{
		NodeB->RemoveEdge(VertexHandleA);
		if (!IslandHandle.IsValid())
		{
			IslandHandle = NodeB->GetParentIsland();
		}
	}

	OnEdgeRemoved.Broadcast(FEdgeSpecifier{VertexHandleA, VertexHandleB});
	
	// Removing an edge should cause the island to check if it needs to split.
	if (Properties.bGenerateIslands && bHandleIslands)
	{
		// Note that we can assume that both nodes are in the same island.
		RemoveOrSplitIsland(IslandHandle.GetIsland());
	}
}

void UGraph::RemoveOrSplitIsland(TObjectPtr<UGraphIsland> Island)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RemoveOrSplitIsland);
	if (!Island || !Island->IsOperationAllowed(EGraphIslandOperations::Split))
	{
		return;
	}

	if (Island->IsEmpty())
	{
		RemoveIsland(Island->Handle());
	}
	else
	{
		TSet<FGraphVertexHandle> IslandNodes = Island->GetVertices();
		TArray<TSet<FGraphVertexHandle>> ConnectedComponents = Graph::Algorithms::FindConnectedComponents(Island->GetVertices());

		// Sort the connected components from largest to smallest since we'll make the assumption that ConnectedComponents[0] will
		// correspond to the already existing island. This way we do the least amount of transferal of nodes from the initial island
		// to the new islands we're going to create.
		ConnectedComponents.Sort(
			[](const TSet<FGraphVertexHandle>& A, const TSet<FGraphVertexHandle>& B)
			{
				return A.Num() > B.Num();
			}
		);

		TArray<TObjectPtr<UGraphIsland>> AffectedIslands = { Island };
		const bool bIsIdentical = ConnectedComponents.Num() == 1 && ConnectedComponents[0].Num() == IslandNodes.Num();
		if (!bIsIdentical)
		{
			// First remove any of the nodes that need to be removed from the existing island.
			for (const FGraphVertexHandle& NodeHandle : IslandNodes)
			{
				if (!ConnectedComponents[0].Contains(NodeHandle))
				{
					Island->RemoveVertex(NodeHandle);
				}
			}

			// Next, create islands for every additional connected component.
			for (int32 Index = 1; Index < ConnectedComponents.Num(); ++Index)
			{
				FGraphIslandHandle NewIsland = CreateIsland(ConnectedComponents[Index].Array());
				if (NewIsland.IsComplete())
				{
					AffectedIslands.Add(NewIsland.GetIsland());
				}
			}
		}

		// All the islands that got changed need to have OnConnectivityChanged called.
		// This let's all the listeners of the islands make a decision on what to do with
		// the island that had a destructive change to its connected components as well as
		// the newly created islands.
		for (UGraphIsland* AffectedIsland : AffectedIslands)
		{
			if (!AffectedIsland)
			{
				continue;
			}

			const EGraphIslandConnectivityChange ChangeType = (AffectedIsland == Island) ? EGraphIslandConnectivityChange::SplitFrom : EGraphIslandConnectivityChange::SplitTo;
			AffectedIsland->HandleOnConnectivityChanged(ChangeType);
		}
	}
}

FGraphVertexHandle UGraph::GetCompleteNodeHandle(const FGraphVertexHandle& InHandle) const
{
	if (!InHandle.IsValid())
	{
		return {};
	}

	const TObjectPtr<UGraphVertex>* Data = Vertices.Find(InHandle);
	if (!Data || !*Data)
	{
		return {};
	}

	return (*Data)->Handle();
}

TObjectPtr<UGraphVertex> UGraph::CreateTypedVertex() const
{
	return NewObject<UGraphVertex>(const_cast<UGraph*>(this), UGraphVertex::StaticClass());
}

TObjectPtr<UGraphIsland> UGraph::CreateTypedIsland() const
{
	return NewObject<UGraphIsland>(const_cast<UGraph*>(this), UGraphIsland::StaticClass());
}

void UGraph::FinalizeVertex(const FGraphVertexHandle& InHandle)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::FinalizeVertex);
	if (!InHandle.IsComplete())
	{
		return;
	}

	TObjectPtr<UGraphVertex> Node = InHandle.GetVertex();
	if (!Node)
	{
		return;
	}

	FGraphIslandHandle IslandHandle = Node->GetParentIsland();
	if (!IslandHandle.IsComplete())
	{
		// Every vertex must be in an island.
		IslandHandle = CreateIsland({ InHandle });
	}

	// Need to iterate through edges to see if any edge was added prior to vertex finalization.
	// If so, we need to do a merge.
	TArray<FEdgeSpecifier> EdgesToMerge;
	Node->ForEachAdjacentVertex(
		[&InHandle, &IslandHandle, &EdgesToMerge](const FGraphVertexHandle& NeighborVertexHandle)
		{
			if (UGraphVertex* NeighborVertex = NeighborVertexHandle.GetVertex())
			{
				const FGraphIslandHandle& NeighborIslandHandle = NeighborVertex->GetParentIsland();
				if (NeighborIslandHandle.IsComplete() && NeighborIslandHandle != IslandHandle)
				{
					EdgesToMerge.Add(FEdgeSpecifier{InHandle, NeighborVertexHandle});
				}
			}
		}
	);

	if (!EdgesToMerge.IsEmpty())
	{
		MergeOrCreateIslands(EdgesToMerge);
	}

	if (TObjectPtr<UGraphIsland> Island = IslandHandle.GetIsland())
	{
		Island->HandleOnConnectivityChanged(EGraphIslandConnectivityChange::VertexAdd);
	}
}

void UGraph::RefreshIslandConnectivity(const FGraphIslandHandle& IslandHandle)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraph::RefreshIslandConnectivity);
	RemoveOrSplitIsland(IslandHandle.GetIsland());
}

UGraphVertex* UGraph::GetSafeVertexFromHandle(const FGraphVertexHandle& Handle) const
{
	return Handle.IsComplete() ? Handle.GetVertex() : Vertices.FindRef(Handle).Get();
}

UGraphIsland* UGraph::GetSafeIslandFromHandle(const FGraphIslandHandle& Handle) const
{
	return Handle.IsComplete() ? Handle.GetIsland() : Islands.FindRef(Handle).Get();
}

void UGraph::ReserveVertices(int32 Delta)
{
	Vertices.Reserve(Vertices.Num() + Delta);
}

void UGraph::ReserveIslands(int32 Delta)
{
	Islands.Reserve(Islands.Num() + Delta);
}

void operator<<(IGraphSerialization& Output, const UGraph& Graph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Gameplay Graph Serialization");

	Output.Initialize(Graph.NumVertices(), Graph::EstimateNumEdges(Graph), Graph.NumIslands());
	Output.WriteGraphProperties(Graph.GetProperties());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Serialize Vertices and Edges");

		for (const TPair<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& Kvp : Graph.GetVertices())
		{
			// Vertices are directly from the graph, keys should always be complete. Checking IsComplete() looks up the key in the map, which is
			// expensive when doing it for every vertex in the graph.
			checkSlow(Kvp.Key.IsComplete());

			if (!ensure(Kvp.Value))
			{
				continue;
			}

			Output.WriteGraphVertex(Kvp.Key, Kvp.Value);

			Kvp.Value->ForEachAdjacentVertex(
				[&Kvp, &Output](const FGraphVertexHandle& AdjacentVertexHandle)
				{
					// Only serialize the vertex in one direction, the reverse is implicit and re-created during deserialization.
					// Note that this permits edges from one vertex back to itself (potentially several such edges), which shall be serialized.
					if (AdjacentVertexHandle < Kvp.Key)
					{
						// This edge is already serialized in the reverse direction as {AdjacentVertexHandle, Kvp.Key}
						return;
					}

					Output.WriteGraphEdge(Kvp.Key, AdjacentVertexHandle);
				}
			);
		}
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Serialize Islands");
		for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Kvp : Graph.GetIslands())
		{
			if (!ensure(Kvp.Key.IsComplete() && Kvp.Value))
			{
				continue;
			}

			Output.WriteGraphIsland(Kvp.Key, Kvp.Value);
		}
	}
}

void operator>>(const IGraphDeserialization& Input, UGraph& Graph)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("Gameplay Graph Deserialization");
	Graph.InitializeFromProperties(Input.GetProperties());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Deserialize Vertices");
		Graph.ReserveVertices(Input.NumVertices());
		Input.ForEveryVertex(
			[&Graph](const FGraphVertexHandle& InHandle)
			{
				if (!InHandle.IsValid())
				{
					return FGraphVertexHandle{};
				}

				return Graph.CreateVertex(InHandle.GetUniqueIndex());
			}
		);
	}

	TArray<FEdgeSpecifier> AllEdges;
	AllEdges.Reserve(Input.NumEdges());
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Deserialize Edges");
		Input.ForEveryEdge(
			[&Graph, &AllEdges](const FEdgeSpecifier& Data)
			{
				const FGraphVertexHandle CompleteHandle1 = Graph.GetCompleteNodeHandle(Data.GetVertexHandle1());
				const FGraphVertexHandle CompleteHandle2 = Graph.GetCompleteNodeHandle(Data.GetVertexHandle2());

				const bool bSuccess = Graph.CreateEdge(
					CompleteHandle1,
					CompleteHandle2,
					false
				);

				if (bSuccess)
				{
					AllEdges.Add(FEdgeSpecifier{CompleteHandle1, CompleteHandle2});
				}
				return bSuccess;
			}
		);
	}

	TArray<FGraphIslandHandle> AllIslands;
	AllIslands.Reserve(Input.NumIslands());

	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Deserialize Islands");
		Graph.ReserveIslands(Input.NumIslands());
		Input.ForEveryIsland(
			[&Graph, &AllIslands](const FGraphIslandHandle& InHandle, const IGraphDeserialization::FIslandConstructionData& Data)
			{
				if (!InHandle.IsValid())
				{
					return FGraphIslandHandle{};
				}

				TArray<FGraphVertexHandle> NewIslandVertices;
				NewIslandVertices.Reserve(Data.Vertices.Num());

				// This is a fallback to handle potentially degenerate cases where serialization
				// has said that a vertex is in more than one island. In that case, we treat each a
				// vertex as having an "implicit edge" to itself. This means that all these islands
				// should be merged instead of creating a new one.
				TArray<FGraphIslandHandle> ExistingIslands;
				UGraphIsland* TargetMergeIsland = nullptr;
		
				for (const FGraphVertexHandle& VertexHandle : Data.Vertices)
				{
					FGraphVertexHandle CompleteHandle = Graph.GetCompleteNodeHandle(VertexHandle);
					if (UGraphVertex* Vertex = CompleteHandle.GetVertex())
					{
						if (UGraphIsland* ParentIsland = Vertex->GetParentIsland().GetIsland())
						{
							ExistingIslands.Add(ParentIsland->Handle());

							if (TargetMergeIsland)
							{
								if (ParentIsland->Num() > TargetMergeIsland->Num())
								{
									TargetMergeIsland = ParentIsland;
								}
							}
							else
							{
								TargetMergeIsland = ParentIsland;
							}
						}
						else
						{
							NewIslandVertices.Add(CompleteHandle);
						}
					}
				}

				if (ExistingIslands.IsEmpty() || !TargetMergeIsland)
				{
					FGraphIslandHandle NewIslandHandle = Graph.CreateIsland(NewIslandVertices, InHandle.GetUniqueIndex());
					AllIslands.Add(NewIslandHandle);
					return NewIslandHandle;
				}
				else
				{
					// Stick all new vertices into the target merge island. Also all the ExistingIslands will get merged into the TargetMergeIsland as well.
					// This handles the case where this new island holds vertices that are different islands already...which would be crazy!
					for (const FGraphVertexHandle& VertexHandle : NewIslandVertices)
					{
						TargetMergeIsland->AddVertex(VertexHandle);
					}

					for (const FGraphIslandHandle& IslandHandle : ExistingIslands)
					{
						if (IslandHandle == TargetMergeIsland->Handle())
						{
							continue;
						}

						if (UGraphIsland* Island = IslandHandle.GetIsland())
						{
							TSet<FGraphVertexHandle> IslandVertices = Island->GetVertices();
							for (const FGraphVertexHandle& VertexHandle : IslandVertices)
							{
								TargetMergeIsland->AddVertex(VertexHandle);
							}

							ensure(Island->Num() == 0);
							Graph.RemoveIsland(IslandHandle);
						}
					}

					// Return a null handle since this is technically not a new island.
					return FGraphIslandHandle{};
				}
			}
		);
	}

	if (Graph::bFixupGraphOnLoad)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("Post Deserialization Fixup");
		Graph.MergeOrCreateIslands(AllEdges);

		for (const FGraphIslandHandle& NewIslandHandle : AllIslands)
		{
			Graph.RemoveOrSplitIsland(NewIslandHandle.GetIsland());
		}
	}
}
