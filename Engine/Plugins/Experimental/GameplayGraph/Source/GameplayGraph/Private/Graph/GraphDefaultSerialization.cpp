// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphDefaultSerialization.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphDefaultSerialization)

bool operator==(const FSerializedEdgeData& Lhs, const FSerializedEdgeData& Rhs)
{
	// We can't really trust that the serialized edge data is in some canonical form so
	// given that we're dealing with undirected graphs, I don't trust that the serialized data
	// will always be in canonical form (where Node1 < Node2) so the equality operation has to
	// take that into account.
	if (Lhs.Node1 == Rhs.Node1)
	{
		return Lhs.Node2 == Rhs.Node2;
	}
	else if (Lhs.Node1 == Rhs.Node2)
	{
		return Lhs.Node2 == Rhs.Node1;
	}

	return false;
}

bool operator!=(const FSerializedEdgeData& Lhs, const FSerializedEdgeData& Rhs)
{
	// Is the default operator!= defined via operator==? In that case this might be redundant.
	return !(Lhs == Rhs);
}

bool operator==(const FSerializedIslandData& Lhs, const FSerializedIslandData& Rhs)
{
	if (Lhs.Vertices.Num() != Rhs.Vertices.Num())
	{
		return false;
	}

	// This should be faster than going through Lhs.Vertices and checking if Rhs.Vertices contains the vertex.
	// In that situation you'd be doing a O(N^2) algorithm since the common case would be the Lhs == Rhs and you'd
	// have to do a linear sweep to find the vertex.
	//
	// In this case, building the set is O(N log N). Then doing N removals from the set is another O(N log N) operation.
	TSet<FGraphVertexHandle> EqualitySet{ Lhs.Vertices };
	ensure(EqualitySet.Num() == Lhs.Vertices.Num());

	for (const FGraphVertexHandle& VertexHandle : Rhs.Vertices)
	{
		if (!EqualitySet.Contains(VertexHandle))
		{
			return false;
		}
	}

	return true;
}

bool operator!=(const FSerializedIslandData& Lhs, const FSerializedIslandData& Rhs)
{
	return !(Lhs == Rhs);
}

bool operator==(const FSerializableGraph& Lhs, const FSerializableGraph& Rhs)
{
	if (Lhs.Properties != Rhs.Properties)
	{
		return false;
	}

	if (Lhs.Vertices.Num() != Rhs.Vertices.Num())
	{
		return false;
	}

	{
		TSet<FGraphVertexHandle> EqualitySet { Lhs.Vertices };
		ensure(EqualitySet.Num() == Lhs.Vertices.Num());
		// Assumptions here: there are no duplicate vertices in Lhs hence why we don't check in the other direction.
		// So given that we've already check the size of the array, this should guarantee equality (barring the duplicate vertex condition).
		for (const FGraphVertexHandle& VertexHandle : Rhs.Vertices)
		{
			if (!EqualitySet.Contains(VertexHandle))
			{
				return false;
			}
		}
	}

	if (Lhs.Edges.Num() != Rhs.Edges.Num())
	{
		return false;
	}

	{
		TSet<FSerializedEdgeData> EqualitySet { Lhs.Edges };
		ensure(EqualitySet.Num() == Lhs.Edges.Num());
		for (const FSerializedEdgeData& EdgeData : Rhs.Edges)
		{
			if (!EqualitySet.Contains(EdgeData))
			{
				return false;
			}
		}
	}

	if (Lhs.Islands.Num() != Rhs.Islands.Num())
	{
		return false;
	}

	for (const TPair<FGraphIslandHandle, FSerializedIslandData>& Kvp : Lhs.Islands)
	{
		const FSerializedIslandData* RhsData = Rhs.Islands.Find(Kvp.Key);
		if (!RhsData)
		{
			return false;
		}

		if (Kvp.Value != *RhsData)
		{
			return false;
		}
	}

	return true;
}