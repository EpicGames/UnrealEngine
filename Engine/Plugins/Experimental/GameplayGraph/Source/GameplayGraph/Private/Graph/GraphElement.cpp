// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphElement.h"
#include "Graph/Graph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphElement)

UGraphElement::UGraphElement(EGraphElementType InElementType)
	: ElementType(InElementType)
{
}

UGraphElement::UGraphElement()
{
}

void UGraphElement::SetParentGraph(TObjectPtr<UGraph> InGraph)
{
	ParentGraph = InGraph;
}

TObjectPtr<UGraph> UGraphElement::GetGraph() const
{
	return ParentGraph.Get();
}
