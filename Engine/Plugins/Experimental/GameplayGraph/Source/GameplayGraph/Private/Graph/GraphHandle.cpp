// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphHandle.h"

#include "Graph/Graph.h"
#include "Graph/GraphElement.h"
#include "Graph/GraphIsland.h"
#include "Graph/GraphVertex.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphHandle)

DEFINE_LOG_CATEGORY(LogGameplayGraph)

FGraphVertexHandle FGraphVertexHandle::Invalid;
FGraphIslandHandle FGraphIslandHandle::Invalid;

FGraphHandle::FGraphHandle(FGraphUniqueIndex InUniqueIndex, UGraph* InGraph)
	: UniqueIndex(InUniqueIndex)
	, WeakGraph(InGraph)
{
}

void FGraphHandle::Clear()
{
	UniqueIndex = FGraphUniqueIndex();
	WeakGraph = nullptr;
}

UGraphVertex* FGraphVertexHandle::GetVertex() const
{
	if (UGraph* Graph = GetGraph())
	{
		return Graph->GetVertices().FindRef(*this);
	}

	return nullptr;
}

bool FGraphVertexHandle::HasElement() const
{
	if (UGraph* Graph = GetGraph())
	{
		return Graph->GetVertices().Contains(*this);
	}

	return false;
}

UGraphIsland* FGraphIslandHandle::GetIsland() const
{
	if (UGraph* Graph = GetGraph())
	{
		return Graph->GetIslands().FindRef(*this);
	}

	return nullptr;
}

bool FGraphIslandHandle::HasElement() const
{
	if (UGraph* Graph = GetGraph())
	{
		return Graph->GetIslands().Contains(*this);
	}

	return false;
}