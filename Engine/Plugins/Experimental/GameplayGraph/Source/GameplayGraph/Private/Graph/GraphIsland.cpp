// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphIsland.h"
#include "Graph/GraphVertex.h"
#include "Graph/Graph.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphIsland)

UGraphIsland::UGraphIsland()
	: Super(EGraphElementType::Island)
{
}

void UGraphIsland::Destroy()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraphIsland::Destroy);
	TSet<FGraphVertexHandle> VertexCopy = Vertices;

	Vertices.Empty();
	// Need to work off a copy of the vertex handles since calling
	// UGraph::RemoveVertex will attempt to remove the node from its
	// parent island as well and that'll run into an error of modifying
	// the Vertices TSet during iteration.
	if (UGraph* Graph = GetGraph())
	{
		Graph->RemoveBulkVertices(VertexCopy.Array());
	}

	HandleOnDestroyed();
}

void UGraphIsland::AddVertex(const FGraphVertexHandle& Node)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraphIsland::AddVertex);
	if (!ensure(Node.IsValid()))
	{
		return;
	}

	UGraphVertex* NodePtr = Node.GetVertex();
	if (!ensure(NodePtr))
	{
		return;
	}

	FGraphIslandHandle OldIslandHandle = NodePtr->GetParentIsland();
	if (OldIslandHandle.IsValid())
	{
		UGraphIsland* OldIsland = OldIslandHandle.GetIsland();
		if (ensure(OldIsland))
		{
			OldIsland->RemoveVertex(Node);
		}
	}

	ensure(NodePtr->GetParentIsland() == FGraphIslandHandle{});
	NodePtr->SetParentIsland(Handle());
	Vertices.Add(Node);
	HandleOnVertexAdded(Node);
}

void UGraphIsland::RemoveVertex(const FGraphVertexHandle& Node)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UGraphIsland::RemoveVertex);
	if (!ensure(Node.IsValid()))
	{
		return;
	}

	const bool bIsInIslandSet = Vertices.Contains(Node);
	bool bIsNodeParentSet = false;

	UGraphVertex* NodePtr = Node.GetVertex();
	if (ensure(NodePtr))
	{
		bIsNodeParentSet = NodePtr->GetParentIsland() == Handle();
		if (bIsNodeParentSet)
		{
			NodePtr->SetParentIsland({});
		}
	}

	if (bIsInIslandSet)
	{
		Vertices.Remove(Node);
	}

	if (bIsInIslandSet || bIsNodeParentSet)
	{
		HandleOnVertexRemoved(Node);
	}
}

void UGraphIsland::ChangeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle)
{
	if (ensure(Vertices.Remove(OldVertexHandle)))
	{
		Vertices.Add(NewVertexHandle);
	}
}

void UGraphIsland::HandleOnVertexAdded(const FGraphVertexHandle& Handle)
{
	OnVertexAdded.Broadcast(this->Handle(), Handle);
}

void UGraphIsland::HandleOnVertexRemoved(const FGraphVertexHandle& Handle)
{
	OnVertexRemoved.Broadcast(this->Handle(), Handle);
}

void UGraphIsland::HandleOnDestroyed()
{
	OnDestroyed.Broadcast(Handle());
}

void UGraphIsland::HandleOnConnectivityChanged(EGraphIslandConnectivityChange Change)
{
	OnConnectivityChanged.Broadcast(Handle(), Change);
}

void UGraphIsland::SetOperationAllowed(EGraphIslandOperations Op, bool bAllowed)
{
	if (bAllowed)
	{
		EnumAddFlags(AllowedOperations, Op);
	}
	else
	{
		EnumRemoveFlags(AllowedOperations, Op);
	}
}