// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/GraphVertex.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GraphVertex)

UGraphVertex::UGraphVertex()
	: Super(EGraphElementType::Node)
{
}

bool UGraphVertex::HasEdgeTo(const FGraphVertexHandle& Other) const
{
	return Edges.Contains(Other);
}

void UGraphVertex::AddEdgeTo(const FGraphVertexHandle& Node)
{
	Edges.Add(Node);
}

void UGraphVertex::RemoveEdge(const FGraphVertexHandle& AdjacentVertexHandle)
{
	UGraphVertex* AdjacentVertex = AdjacentVertexHandle.GetVertex();
	if (ensure(AdjacentVertex))
	{
		AdjacentVertex->Edges.Remove(Handle());
	}

	Edges.Remove(AdjacentVertexHandle);
}

void UGraphVertex::ChangeEdgeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle)
{
	if (ensure(Edges.Remove(OldVertexHandle)))
	{
		Edges.Add(NewVertexHandle);
	}
}

void UGraphVertex::HandleOnVertexRemoved()
{
	OnVertexRemoved.Broadcast(Handle());
}

void UGraphVertex::SetParentIsland(const FGraphIslandHandle& Island)
{
	ParentIsland = Island;
	OnParentIslandSet.Broadcast(Handle(), Island);
}