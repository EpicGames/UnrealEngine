// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/Set.h"
#include "Graph/GraphHandle.h"

namespace Graph::Algorithms
{
	/**
	 * Given a set of nodes, returns an array of connected components.
	 * Note that any nodes that may be otherwise connected to the input nodes will be ignored
	 * for the output if they are not in the input set.
	 */
	TArray<TSet<FGraphVertexHandle>> FindConnectedComponents(const TSet<FGraphVertexHandle>& StartingSet);

}