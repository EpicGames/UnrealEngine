// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/GraphHandle.h"

namespace Graph::Algorithms
{
	/** Returns whether we've found what we've wanted to find (i.e. returning true will end the search). */
	using FSearchCallback = TFunction<bool(const FGraphVertexHandle&)>;

	FGraphVertexHandle BFS(const FGraphVertexHandle& Start, FSearchCallback Callback);
	FGraphVertexHandle DFS(const FGraphVertexHandle& Start, FSearchCallback Callback);
}