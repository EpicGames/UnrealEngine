// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Graph/GraphHandle.h"
#include "Graph/GraphIsland.h"
#include "Graph/GraphVertex.h"
#include "Templates/SubclassOf.h"

#include "Graph.generated.h"

#define UE_API GAMEPLAYGRAPH_API

class IGraphSerialization;
class IGraphDeserialization;

USTRUCT()
struct FGraphProperties
{
	GENERATED_BODY()

	FGraphProperties() = default;
 
	UPROPERTY(SaveGame)
	bool bGenerateIslands = true;
	
	// Comparison operators
	friend bool operator==(const FGraphProperties& Lhs, const FGraphProperties& Rhs) = default;
	friend bool operator!=(const FGraphProperties& Lhs, const FGraphProperties& Rhs) = default;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphVertexCreated, const FGraphVertexHandle&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphIslandCreated, const FGraphIslandHandle&);

// Undirected edge descriptor.
// Always VertexHandle1 <= VertexHandle2.
struct FEdgeSpecifier
{
	FEdgeSpecifier(const FGraphVertexHandle& InVertexHandle1, const FGraphVertexHandle& InVertexHandle2);
	FEdgeSpecifier(const FEdgeSpecifier&) = default;
	FEdgeSpecifier(FEdgeSpecifier&&) = default;
	FEdgeSpecifier& operator=(const FEdgeSpecifier&) = default;
	FEdgeSpecifier& operator=(FEdgeSpecifier&&) = default;

	// Comparison operators
	friend bool operator==(const FEdgeSpecifier& Lhs, const FEdgeSpecifier& Rhs) = default;
	friend bool operator!=(const FEdgeSpecifier& Lhs, const FEdgeSpecifier& Rhs) = default;

	friend uint32 GetTypeHash(const FEdgeSpecifier& InEdge)
	{
		return GetTypeHash(TPair<FGraphVertexHandle, FGraphVertexHandle>{InEdge.VertexHandle1, InEdge.VertexHandle2});
	}

	const FGraphVertexHandle& GetVertexHandle1() const { return VertexHandle1; }
	const FGraphVertexHandle& GetVertexHandle2() const { return VertexHandle2; }

private:
	FGraphVertexHandle VertexHandle1;
	FGraphVertexHandle VertexHandle2;
};

DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphEdgeCreated, const FEdgeSpecifier&);
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphEdgeRemoved, const FEdgeSpecifier&);

/**
 * A UGraph is a collection of nodes and edges. This graph representation
 * is meant to be easily integrable into gameplay systems in the Unreal Engine.
 * 
 * Conceptually, you can imagine that a graph is meant to easily represent relationships
 * so we can answer queries such as:
 *	- Are these two nodes connected to each other?
 *  - How far away are these two nodes?
 *  - Who is the closest node that has XYZ?
 *  - etc.
 * 
 * UGraph provides an interface to be able to run such queries. However, ultimately what
 * makes the graph useful is not only the relationships represented by the edges, but also
 * the data that is stored on each node and each edge. Depending on what the user wants to
 * represent, the user will have to subclass UGraphVertex and UGraphEdge to hold that data.
 * 
 * As the user adds nodes and edges into the graph, they will also be implicitly creating "islands"
 * (i.e. a connected component in the graph). Each graph may have multiple islands. Users
 * of the graph can disable the island detection/creation if needed.
 * 
 * Note that this is an UNDIRECTED GRAPH.
 */
UCLASS(MinimalAPI)
class UGraph: public UObject
{
	GENERATED_BODY()
public:
	UGraph() = default;

	UE_API void Empty();
	UE_API void InitializeFromProperties(const FGraphProperties& Properties);

	/** Given a node handle, find the handle in the current graph with a proper element set. */
	UE_API FGraphVertexHandle GetCompleteNodeHandle(const FGraphVertexHandle& InHandle) const;

	/** Create a node with the specified subclass, adds it to the graph, and returns a handle to it. */
	UE_API FGraphVertexHandle CreateVertex(FGraphUniqueIndex InUniqueIndex = FGraphUniqueIndex::CreateUniqueIndex(false));

	/** Changes a node handle, updating all the references. The vertex must exist. */
	UE_API void ChangeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle);

	/** Creates edges in bulk. This is more efficient than calling CreateEdge multiple times since we will only try to assign a node to an island once. */
	UE_API void CreateBulkEdges(TArray<FEdgeSpecifier>&& NodesToConnect);

	/** Removes a node from the graph along with any edges that contain it. */
	UE_API void RemoveVertex(const FGraphVertexHandle& NodeHandle);

	UE_API void RemoveBulkVertices(const TArray<FGraphVertexHandle>& InHandles);

	/** Removes an edge between two nodes */
	UE_API void RemoveEdge(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB);

	/** This should be called immediately after a node and any relevant edges have been added to the graph. */
	UE_API virtual void FinalizeVertex(const FGraphVertexHandle& InHandle);

	/** Remove an island from the graph. */
	UE_API void RemoveIsland(const FGraphIslandHandle& IslandHandle);

	/** Refresh the connectivity of the given island (re-check to see whether it should be split). */
	UE_API void RefreshIslandConnectivity(const FGraphIslandHandle& IslandHandle);

	int32 NumVertices() const { return Vertices.Num(); }
	int32 NumIslands() const { return Islands.Num(); }

	template<typename TLambda>
	void ForEachIsland(TLambda&& Lambda)
	{
		for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Kvp : Islands)
		{
			Lambda(Kvp.Key, Kvp.Value);
		}
	}

	FOnGraphVertexCreated OnVertexCreated;
	FOnGraphIslandCreated OnIslandCreated;
	FOnGraphEdgeCreated OnEdgeCreated;
	FOnGraphEdgeRemoved OnEdgeRemoved;

	const TMap<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& GetVertices() const { return Vertices; }
	const TMap<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& GetIslands() const { return Islands; }
	const FGraphProperties& GetProperties() const { return Properties; }

	UE_API UGraphVertex* GetSafeVertexFromHandle(const FGraphVertexHandle& Handle) const;
	UE_API UGraphIsland* GetSafeIslandFromHandle(const FGraphIslandHandle& Handle) const;

	GAMEPLAYGRAPH_API friend void operator<<(IGraphSerialization& Output, const UGraph& Graph);
	GAMEPLAYGRAPH_API friend void operator>>(const IGraphDeserialization& Input, UGraph& Graph);

protected:
	/** When we add multiple edges into the graph. This function will ensure that the interactions we make externally are kept to a minimum. */
	UE_API void MergeOrCreateIslands(const TArray<FEdgeSpecifier>& InEdges);

	/** After a change, this function will remove the island if it's empty or will attempt to split it into two smaller islands. */
	UE_API void RemoveOrSplitIsland(TObjectPtr<UGraphIsland> Island);

	/** Used for bulk loading in vertices/edges/islands. */
	UE_API void ReserveVertices(int32 Delta);
	UE_API void ReserveIslands(int32 Delta);
private:
	UPROPERTY()
	TMap<FGraphVertexHandle, TObjectPtr<UGraphVertex>> Vertices;

	UPROPERTY()
	TMap<FGraphIslandHandle, TObjectPtr<UGraphIsland>> Islands;
	FGraphProperties Properties;

	/** Creates an island out of a given set of nodes. */
	UE_API FGraphIslandHandle CreateIsland(TArray<FGraphVertexHandle> Nodes, FGraphUniqueIndex InUniqueIndex = FGraphUniqueIndex());

	/** Creates an edge between the two nodes. */
	UE_API bool CreateEdge(FGraphVertexHandle Node1, FGraphVertexHandle Node2, bool bMergeIslands = true);

	/** Adds a node to the graph's node collection and modifies the NextAvailableNodeUniqueIndex as necessary to maintain its validity. */
	UE_API void RegisterVertex(TObjectPtr<UGraphVertex> Node);

	/** Add an island to the graph and modifies the NextAvailableIslandUniqueIndex as necessary to maintain its validity. */
	UE_API void RegisterIsland(TObjectPtr<UGraphIsland> Edge);

	/** Need this so that users aren't able to pass in bHandleIslands = false. */
	UE_API void RemoveEdgeInternal(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB, bool bHandleIslands);

	/** Factory functions for creating an appropriately typed node/edge/island. */
	UE_API virtual TObjectPtr<UGraphVertex> CreateTypedVertex() const;
	UE_API virtual TObjectPtr<UGraphIsland> CreateTypedIsland() const;
};

#undef UE_API

inline FEdgeSpecifier::FEdgeSpecifier(const FGraphVertexHandle& InVertexHandle1, const FGraphVertexHandle& InVertexHandle2)
	: VertexHandle1(InVertexHandle1)
	, VertexHandle2(InVertexHandle2)
{
	if (VertexHandle2 < VertexHandle1)
	{
		std::swap(VertexHandle1, VertexHandle2);
	}
}
