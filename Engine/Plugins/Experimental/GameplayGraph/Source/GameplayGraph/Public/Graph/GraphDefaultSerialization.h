// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/Graph.h"
#include "Graph/GraphIncrementalSerialization.h"
#include "Graph/GraphSerialization.h"

#include <type_traits>

#include "GraphDefaultSerialization.generated.h"

USTRUCT()
struct FSerializedEdgeData
{
	GENERATED_BODY()

	UPROPERTY(SaveGame)
	FGraphVertexHandle Node1;

	UPROPERTY(SaveGame)
	FGraphVertexHandle Node2;

	// Comparison operators
	friend GAMEPLAYGRAPH_API bool operator==(const FSerializedEdgeData& Lhs, const FSerializedEdgeData& Rhs);
	friend GAMEPLAYGRAPH_API bool operator!=(const FSerializedEdgeData& Lhs, const FSerializedEdgeData& Rhs);

	friend uint32 GetTypeHash(const FSerializedEdgeData& EdgeData)
	{
		return HashCombine(GetTypeHash(EdgeData.Node1), GetTypeHash(EdgeData.Node2));
	}
};

USTRUCT()
struct FSerializedIslandData
{
	GENERATED_BODY()

	UPROPERTY(SaveGame)
	TArray<FGraphVertexHandle> Vertices;

	// Comparison operators
	friend GAMEPLAYGRAPH_API bool operator==(const FSerializedIslandData& Lhs, const FSerializedIslandData& Rhs);
	friend GAMEPLAYGRAPH_API bool operator!=(const FSerializedIslandData& Lhs, const FSerializedIslandData& Rhs);
};

/**
 * The minimum amount of data we need to serialize to be able to reconstruct the graph as it was.
 * Note that classes that inherit from UGraph and its elements will no doubt want to extend the graph
 * with actual information on each node/edge/island. In that case, they should extend FSerializableGraph
 * to contain the extra information per graph handle. Furthermore, they'll need to extend UGraph to have
 * its own typed serialization save/load functions that call the base functions in UGraph first.
 */
USTRUCT()
struct FSerializableGraph
{
	GENERATED_BODY()

	UPROPERTY(SaveGame)
	FGraphProperties Properties;

	UPROPERTY(SaveGame)
	TArray<FGraphVertexHandle> Vertices;

	UPROPERTY(SaveGame)
	TArray<FSerializedEdgeData> Edges;

	UPROPERTY(SaveGame)
	TMap<FGraphIslandHandle, FSerializedIslandData> Islands;

	friend GAMEPLAYGRAPH_API bool operator==(const FSerializableGraph& Lhs, const FSerializableGraph& Rhs);
};

template<typename TSerializableGraph>
class TDefaultGraphSerialization : public IGraphSerialization
{
	static_assert(std::is_base_of_v<FSerializableGraph, TSerializableGraph>, "TDefaultGraphSerialization must serialize a type of FSerializableGraph");
public:
	virtual ~TDefaultGraphSerialization() = default;

	const TSerializableGraph& GetData() const { return Data; }

	virtual void Initialize(int32 NumVertices, int32 NumEdges, int32 NumIslands) override
	{
		Data.Vertices.Reserve(NumVertices);
		Data.Edges.Reserve(NumEdges);
		Data.Islands.Reserve(NumIslands);
	}

	virtual void WriteGraphProperties(const FGraphProperties& Properties) override
	{
		Data.Properties = Properties;
	}

	virtual void WriteGraphVertex(const FGraphVertexHandle& VertexHandle, const UGraphVertex* Vertex) override
	{
		Data.Vertices.Add(VertexHandle);
	}

	virtual void WriteGraphEdge(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB) override
	{
		FSerializedEdgeData Serialized;
		Serialized.Node1 = VertexHandleA;
		Serialized.Node2 = VertexHandleB;
		Data.Edges.Emplace(MoveTemp(Serialized));
	}

	virtual void WriteGraphIsland(const FGraphIslandHandle& IslandHandle, const UGraphIsland* Island) override
	{
		if (!ensure(Island))
		{
			return;
		}

		FSerializedIslandData Serialized;
		for (const FGraphVertexHandle& Handle : Island->GetVertices())
		{
			Serialized.Vertices.Add(Handle);
		}

		Data.Islands.Emplace(IslandHandle, MoveTemp(Serialized));
	}

	virtual void Reset() override
	{
		Data.Vertices.Reset();
		Data.Edges.Reset();
		Data.Islands.Reset();
	}

protected:
	TSerializableGraph Data;
};

template<typename TSerializableGraph>
class TDefaultGraphDeserialization : public IGraphDeserialization
{
	static_assert(std::is_base_of_v<FSerializableGraph, TSerializableGraph>, "TDefaultGraphDeserialization must deserialize a type of FSerializableGraph");
public:
	explicit TDefaultGraphDeserialization(const TSerializableGraph& InData)
	: Data(InData)
	{
	}
	virtual ~TDefaultGraphDeserialization() = default;

	virtual const FGraphProperties& GetProperties() const override
	{
		return Data.Properties;
	}

	virtual int32 NumVertices() const override
	{
		return Data.Vertices.Num();
	}

	virtual void ForEveryVertex(const TFunction<FGraphVertexHandle(const FGraphVertexHandle&)>& Lambda) const override
	{
		for (const FGraphVertexHandle& SerializedHandle : Data.Vertices)
		{
			const FGraphVertexHandle FinalHandle = Lambda(SerializedHandle);
			if (FinalHandle.IsValid())
			{
				OnDeserializedVertex(FinalHandle);
			}
		}
	}

	virtual int32 NumEdges() const override
	{
		return Data.Edges.Num();
	}

	virtual void ForEveryEdge(const TFunction<bool(const FEdgeSpecifier&)>& Lambda) const override
	{
		for (const FSerializedEdgeData& Serialized : Data.Edges)
		{
			const FEdgeSpecifier Construction{ Serialized.Node1, Serialized.Node2 };
			if (Lambda(Construction))
			{
				OnDeserializedEdge(Construction);
			}
		}
	}

	virtual int32 NumIslands() const override
	{
		return Data.Islands.Num();
	}

	virtual void ForEveryIsland(const TFunction<FGraphIslandHandle(const FGraphIslandHandle&, const FIslandConstructionData&)>& Lambda) const override
	{
		for (const TPair<FGraphIslandHandle, FSerializedIslandData>& Serialized : Data.Islands)
		{
			if (Serialized.Value.Vertices.IsEmpty())
			{
				continue;
			}

			FIslandConstructionData Construction;
			Construction.Vertices = Serialized.Value.Vertices;

			const FGraphIslandHandle FinalHandle = Lambda(Serialized.Key, Construction);
			if (FinalHandle.IsValid())
			{
				OnDeserializedIsland(FinalHandle);
			}
		}
	}

protected:
	virtual void OnDeserializedVertex(const FGraphVertexHandle& VertexHandle) const {}
	virtual void OnDeserializedEdge(const FEdgeSpecifier& Edge) const {}
	virtual void OnDeserializedIsland(const FGraphIslandHandle& IslandHandle) const {}

	const TSerializableGraph& Data;
};

using FDefaultGraphSerialization = TDefaultGraphSerialization<FSerializableGraph>;
using FDefaultGraphDeserialization = TDefaultGraphDeserialization<FSerializableGraph>;

//
// DELTA ACTIONS
//
enum class EDefaultDeltaActionType
{
	Add,
	Remove
};

template<EDefaultDeltaActionType Type>
struct TDefaultGraphVertexDeltaAction : public TDerivedGraphDeltaAction<TDefaultGraphVertexDeltaAction<Type>>
{
public:
	explicit TDefaultGraphVertexDeltaAction(const FGraphVertexHandle& InHandle)
	: VertexHandle(InHandle)
	{}

	FGraphVertexHandle VertexHandle;
};

template<EDefaultDeltaActionType Type>
struct TDefaultGraphEdgeDeltaAction : public TDerivedGraphDeltaAction<TDefaultGraphEdgeDeltaAction<Type>>
{
public:
	explicit TDefaultGraphEdgeDeltaAction(const FEdgeSpecifier& InEdge)
	: Edge(InEdge)
	{}

	FEdgeSpecifier Edge;
};

template<EDefaultDeltaActionType Type>
struct TDefaultGraphIslandDeltaAction : public TDerivedGraphDeltaAction<TDefaultGraphIslandDeltaAction<Type>>
{
public:
	explicit TDefaultGraphIslandDeltaAction(const FGraphIslandHandle& InHandle)
	: IslandHandle(InHandle)
	{}

	FGraphIslandHandle IslandHandle;
};


template<EDefaultDeltaActionType Type>
struct TDefaultGraphIslandVertexDeltaAction : public TDerivedGraphDeltaAction<TDefaultGraphIslandVertexDeltaAction<Type>>
{
public:
	explicit TDefaultGraphIslandVertexDeltaAction(const FGraphIslandHandle& InIslandHandle, const FGraphVertexHandle& InVertexHandle)
	: IslandHandle(InIslandHandle)
	, VertexHandle(InVertexHandle)
	{}

	FGraphIslandHandle IslandHandle;
	FGraphVertexHandle VertexHandle;
};

//
// DELTA HANDLER
//
template<typename TInSerializableGraph>
class TDefaultGraphDeltaActionHandler : public TGraphDeltaActionHandler<TInSerializableGraph>
{
	using TThisClass = TDefaultGraphDeltaActionHandler<TInSerializableGraph>;
	static_assert(std::is_base_of_v<FSerializableGraph, TInSerializableGraph>, "TDefaultGraphDeltaActionHandler's TSerializableGraph template parameter must derive from FSerializableGraph.");
public:
	using FCreateVertexDeltaAction = TDefaultGraphVertexDeltaAction<EDefaultDeltaActionType::Add>;
	using FRemoveVertexDeltaAction = TDefaultGraphVertexDeltaAction<EDefaultDeltaActionType::Remove>;

	using FCreateEdgeDeltaAction = TDefaultGraphEdgeDeltaAction<EDefaultDeltaActionType::Add>;
	using FRemoveEdgeDeltaAction = TDefaultGraphEdgeDeltaAction<EDefaultDeltaActionType::Remove>;

	using FCreateIslandDeltaAction = TDefaultGraphIslandDeltaAction<EDefaultDeltaActionType::Add>;
	using FRemoveIslandDeltaAction = TDefaultGraphIslandDeltaAction<EDefaultDeltaActionType::Remove>;

	using FAddIslandVertexDeltaAction = TDefaultGraphIslandVertexDeltaAction<EDefaultDeltaActionType::Add>;
	using FRemoveIslandVertexDeltaAction = TDefaultGraphIslandVertexDeltaAction<EDefaultDeltaActionType::Remove>;

	virtual void InitializeFromGraph(const TInSerializableGraph& InGraph) override
	{
		IncrementalVertices.Append(InGraph.Vertices);
		IncrementalEdges.Append(InGraph.Edges);
	}

	virtual void Flush(TInSerializableGraph& OutGraph) override
	{
		OutGraph.Vertices = IncrementalVertices.Array();
		OutGraph.Edges = IncrementalEdges.Array();
	}

	void Visit(const FCreateVertexDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitCreateVertex);
		IncrementalVertices.Add(Action.VertexHandle);
	}

	void Visit(const FRemoveVertexDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitRemoveVertex);
		IncrementalVertices.Remove(Action.VertexHandle);
	}

	void Visit(const FCreateEdgeDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitCreateEdge);
		FSerializedEdgeData Data;
		Data.Node1 = Action.Edge.GetVertexHandle1();
		Data.Node2 = Action.Edge.GetVertexHandle2();
		IncrementalEdges.Emplace(MoveTemp(Data));
	}

	void Visit(const FRemoveEdgeDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitRemoveEdge);
		FSerializedEdgeData Data;
		Data.Node1 = Action.Edge.GetVertexHandle1();
		Data.Node2 = Action.Edge.GetVertexHandle2();
		IncrementalEdges.Remove(Data);
	}

	void Visit(const FCreateIslandDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitCreateIsland);
		// We're guaranteed that the island created event is fired before vertices are added into it.
		// Hence we're safe to just create an empty FSerializedIslandData here.
		OutGraph.Islands.Emplace(Action.IslandHandle, FSerializedIslandData{});
	}

	void Visit(const FRemoveIslandDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitRemoveIsland);
		OutGraph.Islands.Remove(Action.IslandHandle);
	}

	void Visit(const FAddIslandVertexDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitAddIslandVertex);
		if (FSerializedIslandData* Data = OutGraph.Islands.Find(Action.IslandHandle))
		{
			Data->Vertices.AddUnique(Action.VertexHandle);
		}
	}

	void Visit(const FRemoveIslandVertexDeltaAction& Action, TInSerializableGraph& OutGraph)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(TDefaultGraphDeltaActionHandler::VisitRemoveIslandVertex);
		if (FSerializedIslandData* Data = OutGraph.Islands.Find(Action.IslandHandle))
		{
			Data->Vertices.RemoveSwap(Action.VertexHandle, EAllowShrinking::No);
		}
	}

private:
	// Doing removes from the vertex and edges arrays is too slow so we make sets that we apply operations to that we copy over to the array in Flush.
	TSet<FGraphVertexHandle> IncrementalVertices;
	TSet<FSerializedEdgeData> IncrementalEdges;
};

using FDefaultGraphDeltaActionHandler = TDefaultGraphDeltaActionHandler<FSerializableGraph>;

//
// INCREMENTAL SERIALIZER
//
template<typename TSerializer, typename TDeltaActionHandler>
class TDefaultGraphIncrementalSerialization : public TGraphIncrementalSerialization<TSerializer, TDeltaActionHandler>
{
	static_assert(std::is_base_of_v<TDefaultGraphSerialization<typename TDeltaActionHandler::TSerializableGraph>, TSerializer>, "TDefaultGraphIncrementalSerialization's TSerializer template parameter must derived from TDefaultGraphSerialization.");
	static_assert(std::is_base_of_v<TDefaultGraphDeltaActionHandler<typename TDeltaActionHandler::TSerializableGraph>, TDeltaActionHandler>, "TDefaultGraphIncrementalSerialization's TDeltaActionHandler template parameter should inherit from TDefaultGraphDeltaActionHandler.");
public:

	explicit TDefaultGraphIncrementalSerialization(UGraph* InGraph)
	: TGraphIncrementalSerialization<TSerializer, TDeltaActionHandler>(InGraph)
	{
		if (InGraph)
		{
			InGraph->OnVertexCreated.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphVertexCreated);
			InGraph->OnEdgeCreated.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphEdgeCreated);
			InGraph->OnEdgeRemoved.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphEdgeRemoved);
			InGraph->OnIslandCreated.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphIslandCreated);

			for (const TPair<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& Kvp : InGraph->GetVertices())
			{
				StartListenToVertexChanges(Kvp.Key);
			}

			for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Kvp : InGraph->GetIslands())
			{
				StartListenToIslandChanges(Kvp.Key);
			}
		}
	}

	~TDefaultGraphIncrementalSerialization()
	{
		if (UGraph* Graph = this->GetGraph())
		{
			Graph->OnVertexCreated.RemoveAll(this);
			Graph->OnEdgeCreated.RemoveAll(this);
			Graph->OnEdgeRemoved.RemoveAll(this);
			Graph->OnIslandCreated.RemoveAll(this);

			for (const TPair<FGraphVertexHandle, TObjectPtr<UGraphVertex>>& Kvp : Graph->GetVertices())
			{
				StopListenToVertexChanges(Kvp.Key);
			}

			for (const TPair<FGraphIslandHandle, TObjectPtr<UGraphIsland>>& Kvp : Graph->GetIslands())
			{
				StopListenToIslandChanges(Kvp.Key);
			}
		}
	}

protected:
	virtual void OnGraphVertexCreated(const FGraphVertexHandle& VertexHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FCreateVertexDeltaAction>(VertexHandle);
		StartListenToVertexChanges(VertexHandle);
	}

	virtual void OnGraphVertexRemoved(const FGraphVertexHandle& VertexHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FRemoveVertexDeltaAction>(VertexHandle);
		StopListenToVertexChanges(VertexHandle);
	}

	virtual void OnGraphEdgeCreated(const FEdgeSpecifier& Edge)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FCreateEdgeDeltaAction>(Edge);
	}

	virtual void OnGraphEdgeRemoved(const FEdgeSpecifier& Edge)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FRemoveEdgeDeltaAction>(Edge);
	}

	virtual void OnGraphIslandCreated(const FGraphIslandHandle& IslandHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FCreateIslandDeltaAction>(IslandHandle);
		StartListenToIslandChanges(IslandHandle);
	}

	virtual void OnGraphIslandRemoved(const FGraphIslandHandle& IslandHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FRemoveIslandDeltaAction>(IslandHandle);
		StopListenToIslandChanges(IslandHandle);
	}

	virtual void OnGraphIslandVertexAdded(const FGraphIslandHandle& IslandHandle, const FGraphVertexHandle& VertexHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FAddIslandVertexDeltaAction>(IslandHandle, VertexHandle);
	}

	virtual void OnGraphIslandVertexRemoved(const FGraphIslandHandle& IslandHandle, const FGraphVertexHandle& VertexHandle)
	{
		this->template AddDeltaAction<typename TDeltaActionHandler::FRemoveIslandVertexDeltaAction>(IslandHandle, VertexHandle);
	}

	virtual void StartListenToVertexChanges(const FGraphVertexHandle& VertexHandle)
	{
		if (UGraphVertex* Vertex = VertexHandle.GetVertex())
		{
			Vertex->OnVertexRemoved.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphVertexRemoved);
		}
	}

	virtual void StopListenToVertexChanges(const FGraphVertexHandle& VertexHandle)
	{
		if (UGraphVertex* Vertex = VertexHandle.GetVertex())
		{
			Vertex->OnVertexRemoved.RemoveAll(this);
		}
	}

	virtual void StartListenToIslandChanges(const FGraphIslandHandle& IslandHandle)
	{
		if (UGraphIsland* Island = IslandHandle.GetIsland())
		{
			Island->OnDestroyed.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphIslandRemoved);
			Island->OnVertexAdded.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphIslandVertexAdded);
			Island->OnVertexRemoved.AddRaw(this, &TDefaultGraphIncrementalSerialization::OnGraphIslandVertexRemoved);
		}
	}

	virtual void StopListenToIslandChanges(const FGraphIslandHandle& IslandHandle)
	{
		if (UGraphIsland* Island = IslandHandle.GetIsland())
		{
			Island->OnDestroyed.RemoveAll(this);
			Island->OnVertexAdded.RemoveAll(this);
			Island->OnVertexRemoved.RemoveAll(this);
		}
	}
};

using FDefaultGraphIncrementalSerialization = TDefaultGraphIncrementalSerialization<FDefaultGraphSerialization, FDefaultGraphDeltaActionHandler>;