// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "UObject/Object.h"
#include "Graph/GraphHandle.h"

#include "GraphElement.generated.h"

#define UE_API GAMEPLAYGRAPH_API

class UGraph;

UENUM()
enum class EGraphElementType
{
	Node,
	Edge,
	Island,
	Unknown
};

UCLASS(MinimalAPI, abstract)
class UGraphElement : public UObject
{
	GENERATED_BODY()
public:
	UE_API explicit UGraphElement(EGraphElementType InElementType);
	EGraphElementType GetElementType() const { return ElementType; }

	friend class UGraph;
protected:
	UE_API UGraphElement();

	void SetUniqueIndex(FGraphUniqueIndex InUniqueIndex) { UniqueIndex = InUniqueIndex; }
	FGraphUniqueIndex GetUniqueIndex() const { return UniqueIndex; }

	UE_API void SetParentGraph(TObjectPtr<UGraph> InGraph);
	UE_API TObjectPtr<UGraph> GetGraph() const;

	/** Called when we create this element and prior to setting any properties. */
	virtual void OnCreate() {}

private:
	UPROPERTY()
	EGraphElementType ElementType = EGraphElementType::Unknown;

	/** Will match the UniqueIndex in the UGraphHandle that references this element. */
	UPROPERTY()
	FGraphUniqueIndex UniqueIndex = FGraphUniqueIndex();

	UPROPERTY()
	TWeakObjectPtr<UGraph> ParentGraph;
};

#undef UE_API
