// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/Guid.h"
#include "UObject/Object.h"

#include "GraphHandle.generated.h"

#define UE_API GAMEPLAYGRAPH_API

class UGraph;
class UGraphVertex;
class UGraphIsland;

DECLARE_LOG_CATEGORY_EXTERN(LogGameplayGraph, Log, All);

USTRUCT()
struct FGraphUniqueIndex
{
	GENERATED_BODY()
public:
	FGraphUniqueIndex(bool InIsTemp = false)
		: UniqueIndex(FGuid())
		, bIsTemporary(InIsTemp) 
	{}

	FGraphUniqueIndex(FGuid InUniqueIndex, bool InIsTemp = false)
		: UniqueIndex(InUniqueIndex)
		, bIsTemporary(InIsTemp) 
	{}


	bool IsValid() const
	{
		return UniqueIndex.IsValid();
	}

	bool IsTemporary() const
	{
		return bIsTemporary;
	}

	void SetTemporary(bool InTemp)
	{
		bIsTemporary = InTemp;
	}

	FGraphUniqueIndex NextUniqueIndex() const
	{
		return FGraphUniqueIndex(FGuid::NewGuid(),bIsTemporary);
	}

	static FGraphUniqueIndex CreateUniqueIndex(bool InIsTemp = false)
	{
		return FGraphUniqueIndex(FGuid::NewGuid(), InIsTemp);
	}

	bool operator==(const FGraphUniqueIndex& Other) const
	{
		return UniqueIndex == Other.UniqueIndex;
	}

	bool operator!=(const FGraphUniqueIndex& Other) const
	{
		return UniqueIndex != Other.UniqueIndex;
	}

	bool operator<(const FGraphUniqueIndex& Other) const
	{
		return UniqueIndex < Other.UniqueIndex;
	}

	friend uint32 GetTypeHash(const FGraphUniqueIndex& InUniqueIndex)
	{
		return uint32(CityHash64((char*)&InUniqueIndex.UniqueIndex, sizeof(FGuid)));
	}

	FString ToString() const
	{
		return UniqueIndex.ToString();
	}


private:
	/** Unique identifier within a graph. */
	UPROPERTY(SaveGame)
	FGuid UniqueIndex = FGuid();

	/** Temporary Status for index */
	UPROPERTY(Transient)
	bool bIsTemporary = false;

};


/**
 * For persistence, every node in a graph is given a unique index.
 * A FGraphHandle encapsulates that index to make it easy to go from
 * the index to the node and vice versa.
 */
USTRUCT()
struct FGraphHandle
{
	GENERATED_BODY()
public:
	FGraphHandle() = default;
	UE_API FGraphHandle(FGraphUniqueIndex InUniqueIndex, UGraph* InGraph);
	virtual ~FGraphHandle() = default;

	UE_API void Clear();

	/** Whether or not this handle has been initialized. */
	bool IsValid() const;
	bool IsComplete() const;
	virtual bool HasElement() const { return false; }

	FGraphUniqueIndex GetUniqueIndex() const { return UniqueIndex; }
	UGraph* GetGraph() const { return WeakGraph.Get(); }

	bool operator==(const FGraphHandle& Other) const;
	bool operator!=(const FGraphHandle& Other) const;
	bool operator<(const FGraphHandle& Other) const;

	friend uint32 GetTypeHash(const FGraphHandle& Handle);
private:
	/** Unique identifier within a graph. */
	UPROPERTY(SaveGame)
	FGraphUniqueIndex UniqueIndex = FGraphUniqueIndex();

	/** Pointer to the graph */
	UPROPERTY(Transient)
	TWeakObjectPtr<UGraph> WeakGraph;
};

USTRUCT()
struct FGraphVertexHandle : public FGraphHandle
{
	GENERATED_BODY()

	static UE_API FGraphVertexHandle Invalid;

	FGraphVertexHandle() = default;
	FGraphVertexHandle(FGraphUniqueIndex InUniqueIndex, UGraph* InGraph)
		: FGraphHandle(InUniqueIndex, InGraph)
	{}

	UE_API UGraphVertex* GetVertex() const;
	UE_API virtual bool HasElement() const override;
};

USTRUCT()
struct FGraphIslandHandle : public FGraphHandle
{
	GENERATED_BODY()

	static UE_API FGraphIslandHandle Invalid;

	FGraphIslandHandle() = default;
	FGraphIslandHandle(FGraphUniqueIndex InUniqueIndex, UGraph* InGraph)
		: FGraphHandle(InUniqueIndex, InGraph)
	{}

	UE_API UGraphIsland* GetIsland() const;
	UE_API virtual bool HasElement() const override;
};

#undef UE_API

inline bool FGraphHandle::IsValid() const
{
	return UniqueIndex.IsValid();
}

inline bool FGraphHandle::IsComplete() const
{
	return IsValid() && HasElement();
}

inline bool FGraphHandle::operator==(const FGraphHandle& Other) const
{
	return UniqueIndex == Other.UniqueIndex;
}

inline bool FGraphHandle::operator!=(const FGraphHandle& Other) const
{
	return UniqueIndex != Other.UniqueIndex;
}

inline bool FGraphHandle::operator<(const FGraphHandle& Other) const
{
	return UniqueIndex < Other.UniqueIndex;
}

inline uint32 GetTypeHash(const FGraphHandle& Handle)
{
	return GetTypeHash(Handle.UniqueIndex);
}
