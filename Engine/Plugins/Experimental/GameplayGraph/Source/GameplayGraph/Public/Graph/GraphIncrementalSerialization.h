// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/Graph.h"

/*
 * =========== README FIRST ===========
 *
 * This file provides the basics by which you can support incremental serialization for a UGraph for any possible
 * serialized type. There are a few concepts at work here:
 * 	- The "Graph" - this is the gameplay graph that you wish to serialize.
 *  - The "Serializer" - this is the class that inherits from IGraphSerialization that serializes your data.
 * 	  There is an additional caveat: this class must implement a "GetData" function that returns (see next point)...
 *  - The "Serialized Graph Data" - this is the object that you wish to end up with after serialization.
 * 
 * With incremental serialization we introduce:
 *  - "Delta Actions" - Any time the graph is changed, we add a "Delta Action" to a buffer. The next time Flush is called
 *    on the incremental serialization, we'd expect the actions in the buffer to get applied to the cached data and make it
 *    equivalent to the graph's current state.
 *  - The "Delta Action Handler" - This takes in a list of "Delta Actions" and makes the appropriate changes to the "Serialized Graph Data".
 *  - The "TGraphIncrementalSerialization" - this is the public interface to all of the above. Users of incremental serialization
 *    need only construct a subclass of this to use. It should under-the-hood hook up to the graph events to properly detect changes.
 */

template<typename TDerived>
struct TDerivedGraphDeltaAction
{
public:
	template<typename TDeltaActionHandler>
	void Accept(TDeltaActionHandler& Visitor, typename TDeltaActionHandler::TSerializableGraph& OutGraph) const
	{
		Visitor.Visit(*static_cast<const TDerived*>(this), OutGraph);
	}
};

template<typename TInSerializableGraph>
class TGraphDeltaActionHandler
{
	template<typename T>
	struct AlwaysFalse
	{
	public:
		static constexpr bool False = false;
	};
public:
	using TSerializableGraph = TInSerializableGraph;

	TGraphDeltaActionHandler()
	{
	}

	virtual ~TGraphDeltaActionHandler() = default;

	virtual void InitializeFromGraph(const TSerializableGraph& InGraph) {}
	virtual void Flush(TSerializableGraph& OutGraph) {}

	// Catch-all to remind users if their delta action handler doesn't actually support an action they created.
	template<typename TAction>
	void Visit(const TAction& Action, TSerializableGraph& OutGraph)
	{
		static_assert(AlwaysFalse<TAction>::False, "Unimplemented Visit function.");
	}
};

template<typename TSerializer, typename TDeltaActionHandler>
class TGraphIncrementalSerialization
{
public:
	virtual ~TGraphIncrementalSerialization() = default;

	explicit TGraphIncrementalSerialization(UGraph* InGraph)
	: Graph(InGraph)
	{
		if (!ensure(InGraph))
		{
			return;
		}
		// Build the serialized graph data from the current graph state and only do incremental updates from here on out.
		TSerializer Serializer;
		Serializer << *InGraph;
		SerializedGraphData = Serializer.GetData();
		DeltaActionHandler.InitializeFromGraph(SerializedGraphData);
	}

	/**
     * This is not a const function as derived classes as the DeltaActionHandler may need to do a final pass on buffered data for example.
     */
	const typename TDeltaActionHandler::TSerializableGraph& GetLatestData()
	{
		DeltaActionHandler.Flush(SerializedGraphData);
		return SerializedGraphData;
	}

protected:
	template<typename TDeltaAction, typename... TParams>
	void AddDeltaAction(TParams... Params)
	{
		TDeltaAction DeltaAction{Params...};
		DeltaAction.Accept(DeltaActionHandler, SerializedGraphData);
	}

	UGraph* GetGraph() const
	{
		return Graph.Get();
	}

private:
	TWeakObjectPtr<UGraph> Graph;
	typename TDeltaActionHandler::TSerializableGraph SerializedGraphData;
	// Note that the assumption here is that TDeltaActionHandler is the most derived type already so we don't need to use a pointer here.
	TDeltaActionHandler DeltaActionHandler;
};
