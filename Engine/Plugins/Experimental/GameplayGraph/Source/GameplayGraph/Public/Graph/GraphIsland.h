// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Misc/EnumClassFlags.h"
#include "Graph/GraphElement.h"
#include "GraphIsland.generated.h"

#define UE_API GAMEPLAYGRAPH_API

/**
 * These are the possible operations that can be done to an island. The graph
 * will attempt to check that the island is allowing these operations before
 * successfully performing any of these operations. By default all operations are allowed.
 */
UENUM()
enum class EGraphIslandOperations : int32
{
	None = 0,
	Add = 1 << 0,
	Split = 1 << 1,
	Merge = 1 << 2,
	Destroy = 1 << 3,
	All = Add | Split | Merge | Destroy
};
ENUM_CLASS_FLAGS(EGraphIslandOperations);

UENUM()
enum class EGraphIslandConnectivityChange : int32
{
	// Vertex added into an island
	VertexAdd,
	// An island is split into 2 more islands
	SplitFrom,
	// An island was created by splitting an old island.
	SplitTo,
	// Some other undefined change - not used by the library but can be used by external users as a no-op of sorts.
	Other
};

/** Delegate to track when some sort of batch change has occurred on this island that probably changes its connectivity.
 *  This is different from FOnGraphIslandNodeRemoved since FOnGraphIslandDestructiveChangeFinish will only be called
 *  once for the graph for a given operation while FOnGraphIslandNodeRemoved may be called multiple times if we're removing
 *  more than one node from the island at a given time. Note that this will only be called as a result of a destructive change.
 *  So repeatedly adding a node to an island won't call this event. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGraphIslandConnectedComponentsChanged, const FGraphIslandHandle&, EGraphIslandConnectivityChange);

/** Delegate to track when this island should no longer exist. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphIslandDestroyed, const FGraphIslandHandle&);

/** Delegate to track the event when the island has a node added to it. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGraphIslandVertexAdded, const FGraphIslandHandle&, const FGraphVertexHandle&);

/** Delegate to track the event when the island has a node removed from it. */
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGraphIslandVertexRemoved, const FGraphIslandHandle&, const FGraphVertexHandle&);

UCLASS(MinimalAPI)
class UGraphIsland : public UGraphElement
{
	GENERATED_BODY()
public:
	UE_API UGraphIsland();

	FGraphIslandHandle Handle() const
	{
		return FGraphIslandHandle{ GetUniqueIndex(), GetGraph() };
	}

	bool IsEmpty() const { return Vertices.IsEmpty(); }
	const TSet<FGraphVertexHandle>& GetVertices() const { return Vertices; }
	int32 Num() const { return Vertices.Num(); }

	bool IsOperationAllowed(EGraphIslandOperations Op) const { return EnumHasAnyFlags(AllowedOperations, Op); }
	UE_API void SetOperationAllowed(EGraphIslandOperations Op, bool bAllowed);

	template<typename TLambda>
	void ForEachVertex(TLambda&& Lambda)
	{
		for (const FGraphVertexHandle& Vh : Vertices)
		{
			Lambda(Vh);
		}
	}

	/** Adds a single node into this island. */
	UE_API void AddVertex(const FGraphVertexHandle& Node);

	/** Removes a node from the island. */
	UE_API void RemoveVertex(const FGraphVertexHandle& Node);

	/** Changes a node handle. The vertex must exist. */
	UE_API virtual void ChangeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle);

	FOnGraphIslandVertexAdded OnVertexAdded;
	FOnGraphIslandVertexRemoved OnVertexRemoved;
	FOnGraphIslandDestroyed OnDestroyed;
	FOnGraphIslandConnectedComponentsChanged OnConnectivityChanged;

	friend class UGraph;
protected:

	UE_API virtual void HandleOnVertexAdded(const FGraphVertexHandle& Handle);
	UE_API virtual void HandleOnVertexRemoved(const FGraphVertexHandle& Handle);
	UE_API virtual void HandleOnDestroyed();
	UE_API virtual void HandleOnConnectivityChanged(EGraphIslandConnectivityChange Change);

	/** Called when removing the island from the graph. */
	UE_API void Destroy();

private:
	UPROPERTY(SaveGame)
	TSet<FGraphVertexHandle> Vertices;

	UPROPERTY(Transient)
	EGraphIslandOperations AllowedOperations = EGraphIslandOperations::All;
};

#undef UE_API
