// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/GraphHandle.h"

class UGraphVertex;
class UGraphIsland;

struct FEdgeSpecifier;
struct FGraphProperties;
struct FGraphVertexHandle;
struct FGraphIslandHandle;

class IGraphSerialization
{
public:
	virtual ~IGraphSerialization() = default;

	virtual void Initialize(int32 NumVertices, int32 NumEdges, int32 NumIslands) = 0;
	virtual void WriteGraphProperties(const FGraphProperties& Properties) = 0;
	virtual void WriteGraphVertex(const FGraphVertexHandle& VertexHandle, const UGraphVertex* Vertex) = 0;
	virtual void WriteGraphEdge(const FGraphVertexHandle& VertexHandleA, const FGraphVertexHandle& VertexHandleB) = 0;
	virtual void WriteGraphIsland(const FGraphIslandHandle& IslandHandle, const UGraphIsland* Island) = 0;
	virtual void Reset() = 0;
};

class IGraphDeserialization
{
public:
	virtual ~IGraphDeserialization() = default;

	virtual const FGraphProperties& GetProperties() const = 0;
	virtual int32 NumVertices() const = 0;
	virtual void ForEveryVertex(const TFunction<FGraphVertexHandle(const FGraphVertexHandle&)>& Lambda) const = 0;

	virtual int32 NumEdges() const = 0;
	virtual void ForEveryEdge(const TFunction<bool(const FEdgeSpecifier&)>& Lambda) const = 0;

	struct FIslandConstructionData
	{
		TArray<FGraphVertexHandle> Vertices;
	};

	virtual int32 NumIslands() const = 0;
	virtual void ForEveryIsland(const TFunction<FGraphIslandHandle(const FGraphIslandHandle&, const FIslandConstructionData&)>& Lambda) const = 0;
};
