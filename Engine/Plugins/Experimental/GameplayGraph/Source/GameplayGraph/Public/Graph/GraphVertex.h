// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Graph/GraphElement.h"

#include "GraphVertex.generated.h"

#define UE_API GAMEPLAYGRAPH_API

/** Event for when the node has been removed from the graph. */
DECLARE_MULTICAST_DELEGATE_OneParam(FOnGraphVertexRemoved, const FGraphVertexHandle&);
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGraphVertexParentIslandSet, const FGraphVertexHandle&, const FGraphIslandHandle&);

UCLASS(MinimalAPI)
class UGraphVertex : public UGraphElement
{
	GENERATED_BODY()
public:
	UE_API UGraphVertex();

	FGraphVertexHandle Handle() const
	{
		return FGraphVertexHandle{ GetUniqueIndex(), GetGraph()};
	}

	UE_API bool HasEdgeTo(const FGraphVertexHandle& Other) const;
	int32 NumEdges() const { return Edges.Num(); }

	const FGraphIslandHandle& GetParentIsland() const { return ParentIsland; }

	template<typename TLambda>
	void ForEachAdjacentVertex(TLambda&& Lambda)
	{
		for (const FGraphVertexHandle& Vertex : Edges)
		{
			Lambda(Vertex);
		}
	}
	
	const TSet<FGraphVertexHandle>& GetEdges() const { return Edges; }

	/** Changes an edge vertex handle. The vertex must exist. */
	UE_API void ChangeEdgeVertexHandle(const FGraphVertexHandle& OldVertexHandle, const FGraphVertexHandle& NewVertexHandle);

	FOnGraphVertexRemoved OnVertexRemoved;
	FOnGraphVertexParentIslandSet OnParentIslandSet;

	friend class UGraph;
	friend class UGraphIsland;
protected:

	UE_API void AddEdgeTo(const FGraphVertexHandle& Node);
	UE_API void RemoveEdge(const FGraphVertexHandle& AdjacentVertexHandle);
	UE_API void SetParentIsland(const FGraphIslandHandle& Island);

	UE_API virtual void HandleOnVertexRemoved();
private:

	UPROPERTY()
	TSet<FGraphVertexHandle> Edges;

	UPROPERTY()
	FGraphIslandHandle ParentIsland;
};

#undef UE_API
